// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpDigestAuth.h"
#include "Misc/SecureHash.h"
#include "Misc/Guid.h"

bool FHttpDigestAuthHelper::ParseDigestChallenge(const FString& AuthHeader, FDigestChallenge& OutChallenge)
{
	// WWW-Authenticate: Digest realm="...", nonce="...", qop="auth", ...

	// Remove "Digest " prefix if present
	FString Header = AuthHeader;
	if (Header.StartsWith(TEXT("Digest "), ESearchCase::IgnoreCase))
	{
		Header = Header.RightChop(7).TrimStart();
	}

	// Split by comma (but be careful of quoted values)
	TArray<FString> Parts;
	int32 Start = 0;
	bool bInQuotes = false;

	for (int32 i = 0; i < Header.Len(); i++)
	{
		if (Header[i] == '"')
		{
			bInQuotes = !bInQuotes;
		}
		else if (Header[i] == ',' && !bInQuotes)
		{
			Parts.Add(Header.Mid(Start, i - Start).TrimStartAndEnd());
			Start = i + 1;
		}
	}
	// Add last part
	if (Start < Header.Len())
	{
		Parts.Add(Header.Mid(Start).TrimStartAndEnd());
	}

	// Parse each key=value pair
	for (const FString& Part : Parts)
	{
		FString Key, Value;
		if (ParseHeaderPair(Part, Key, Value))
		{
			if (Key.Equals(TEXT("realm"), ESearchCase::IgnoreCase))
			{
				OutChallenge.Realm = Value;
			}
			else if (Key.Equals(TEXT("nonce"), ESearchCase::IgnoreCase))
			{
				OutChallenge.Nonce = Value;
			}
			else if (Key.Equals(TEXT("qop"), ESearchCase::IgnoreCase))
			{
				OutChallenge.Qop = Value;
			}
			else if (Key.Equals(TEXT("opaque"), ESearchCase::IgnoreCase))
			{
				OutChallenge.Opaque = Value;
			}
			else if (Key.Equals(TEXT("algorithm"), ESearchCase::IgnoreCase))
			{
				OutChallenge.Algorithm = Value;
			}
		}
	}

	// Realm and Nonce are required
	return !OutChallenge.Realm.IsEmpty() && !OutChallenge.Nonce.IsEmpty();
}

FString FHttpDigestAuthHelper::GenerateDigestResponse(
	const FString& Username,
	const FString& Password,
	const FString& Method,
	const FString& URI,
	const FDigestChallenge& Challenge)
{
	// HA1 = MD5(username:realm:password)
	FString HA1Input = FString::Printf(TEXT("%s:%s:%s"), *Username, *Challenge.Realm, *Password);
	FString HA1 = ComputeMD5Hash(HA1Input);

	// HA2 = MD5(method:uri)
	FString HA2Input = FString::Printf(TEXT("%s:%s"), *Method, *URI);
	FString HA2 = ComputeMD5Hash(HA2Input);

	FString Response;
	FString AuthValue;

	// Check if qop is specified
	if (!Challenge.Qop.IsEmpty())
	{
		// qop-based authentication (RFC 2617)
		FString CNonce = GenerateNonce();
		FString NonceCount = TEXT("00000001");

		// Response = MD5(HA1:nonce:nc:cnonce:qop:HA2)
		FString ResponseInput = FString::Printf(
			TEXT("%s:%s:%s:%s:%s:%s"),
			*HA1, *Challenge.Nonce, *NonceCount, *CNonce, *Challenge.Qop, *HA2);
		Response = ComputeMD5Hash(ResponseInput);

		// Build Authorization header
		AuthValue = FString::Printf(
			TEXT("Digest username=\"%s\", realm=\"%s\", nonce=\"%s\", uri=\"%s\", ")
			TEXT("qop=%s, nc=%s, cnonce=\"%s\", response=\"%s\""),
			*Username, *Challenge.Realm, *Challenge.Nonce, *URI,
			*Challenge.Qop, *NonceCount, *CNonce, *Response);
	}
	else
	{
		// Legacy mode without qop
		// Response = MD5(HA1:nonce:HA2)
		FString ResponseInput = FString::Printf(TEXT("%s:%s:%s"), *HA1, *Challenge.Nonce, *HA2);
		Response = ComputeMD5Hash(ResponseInput);

		// Build Authorization header
		AuthValue = FString::Printf(
			TEXT("Digest username=\"%s\", realm=\"%s\", nonce=\"%s\", uri=\"%s\", response=\"%s\""),
			*Username, *Challenge.Realm, *Challenge.Nonce, *URI, *Response);
	}

	// Add opaque if present
	if (!Challenge.Opaque.IsEmpty())
	{
		AuthValue += FString::Printf(TEXT(", opaque=\"%s\""), *Challenge.Opaque);
	}

	return AuthValue;
}

FString FHttpDigestAuthHelper::ComputeMD5Hash(const FString& Input)
{
	// Use Unreal's FMD5 for hash computation
	FMD5 MD5Gen;

	// Convert to UTF-8 for hashing
	FTCHARToUTF8 UTF8String(*Input);
	MD5Gen.Update(reinterpret_cast<const uint8*>(UTF8String.Get()), UTF8String.Length());

	uint8 Digest[16];
	MD5Gen.Final(Digest);

	// Convert to lowercase hex string
	FString Result;
	for (int32 i = 0; i < 16; i++)
	{
		Result += FString::Printf(TEXT("%02x"), Digest[i]);
	}

	return Result;
}

FString FHttpDigestAuthHelper::GenerateNonce()
{
	// Generate random nonce using GUID
	FGuid Guid = FGuid::NewGuid();
	return Guid.ToString(EGuidFormats::Digits);
}

bool FHttpDigestAuthHelper::ParseHeaderPair(const FString& Pair, FString& OutKey, FString& OutValue)
{
	int32 EqualPos;
	if (!Pair.FindChar('=', EqualPos))
	{
		return false;
	}

	OutKey = Pair.Left(EqualPos).TrimStartAndEnd();
	OutValue = Pair.Mid(EqualPos + 1).TrimStartAndEnd();

	// Remove quotes if present
	if (OutValue.StartsWith(TEXT("\"")) && OutValue.EndsWith(TEXT("\"")))
	{
		OutValue = OutValue.Mid(1, OutValue.Len() - 2);
	}

	return true;
}
