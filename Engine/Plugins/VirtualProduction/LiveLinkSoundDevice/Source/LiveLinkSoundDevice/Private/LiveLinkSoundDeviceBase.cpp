// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkSoundDeviceBase.h"
#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "LiveLinkHub/ILiveLinkRecordingSession.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "Misc/DateTime.h"

#define LOCTEXT_NAMESPACE "LiveLinkSoundDeviceRecorder"

//~ Begin ULiveLinkDevice interface

TSubclassOf<ULiveLinkDeviceSettings> ULiveLinkSoundDeviceBase::GetSettingsClass() const
{
	return SettingsClass ? SettingsClass : TSubclassOf<ULiveLinkDeviceSettings>(ULiveLinkSoundDeviceSettings::StaticClass());
}

EDeviceHealth ULiveLinkSoundDeviceBase::GetDeviceHealth() const
{
	switch (ConnectionStatus)
	{
	case ELiveLinkDeviceConnectionStatus::Connected:
		return bIsRecording ? EDeviceHealth::Good : EDeviceHealth::Nominal;
	case ELiveLinkDeviceConnectionStatus::Connecting:
		return EDeviceHealth::Info;
	case ELiveLinkDeviceConnectionStatus::Disconnecting:
		return EDeviceHealth::Warning;
	case ELiveLinkDeviceConnectionStatus::Disconnected:
	default:
		return EDeviceHealth::Error;
	}
}

FText ULiveLinkSoundDeviceBase::GetHealthText() const
{
	switch (ConnectionStatus)
	{
	case ELiveLinkDeviceConnectionStatus::Connected:
		return bIsRecording ? LOCTEXT("Recording", "Recording") : LOCTEXT("Connected", "Connected - Ready");
	case ELiveLinkDeviceConnectionStatus::Connecting:
		return LOCTEXT("Connecting", "Connecting...");
	case ELiveLinkDeviceConnectionStatus::Disconnecting:
		return LOCTEXT("Disconnecting", "Disconnecting...");
	case ELiveLinkDeviceConnectionStatus::Disconnected:
	default:
		return LOCTEXT("Disconnected", "Disconnected");
	}
}

void ULiveLinkSoundDeviceBase::OnDeviceAdded()
{
	// Subscribe to recording session delegates for slate/take changes
	ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
	Session.OnSlateNameChanged().AddUObject(this, &ULiveLinkSoundDeviceBase::HandleSlateNameChanged);
	Session.OnTakeNumberChanged().AddUObject(this, &ULiveLinkSoundDeviceBase::HandleTakeNumberChanged);

	// Start main device ticker (handles reconnection and polling)
	DeviceTickerHandle = FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateUObject(this, &ULiveLinkSoundDeviceBase::DeviceTick),
		PollingIntervalSeconds
	);

	ULiveLinkSoundDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkSoundDeviceSettings>();
	DeviceSettings->DisplayName = FString::Printf(TEXT("Sound Devices (%s)"), *DeviceSettings->IpAddress);
}

void ULiveLinkSoundDeviceBase::OnDeviceRemoved()
{
	// Unsubscribe from recording session delegates
	ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
	Session.OnSlateNameChanged().RemoveAll(this);
	Session.OnTakeNumberChanged().RemoveAll(this);

	// Stop device ticker
	if (DeviceTickerHandle.IsValid())
	{
		FTSTicker::GetCoreTicker().RemoveTicker(DeviceTickerHandle);
		DeviceTickerHandle.Reset();
	}

	if (ConnectionStatus != ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		Disconnect_Implementation();
	}
}

void ULiveLinkSoundDeviceBase::OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkDeviceSettings, DisplayName))
	{
		return;
	}

	// When settings change, attempt immediate reconnection if disconnected
	if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		UE_LOGF(LogTemp, Log, "Settings changed, attempting connection to Sound Devices...");
		AttemptReconnection();
	}
}

//~ End ULiveLinkDevice interface

//~ Begin ILiveLinkDeviceCapability_Connection interface

ELiveLinkDeviceConnectionStatus ULiveLinkSoundDeviceBase::GetConnectionStatus_Implementation() const
{
	return ConnectionStatus;
}

FString ULiveLinkSoundDeviceBase::GetHardwareId_Implementation() const
{
	const ULiveLinkSoundDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkSoundDeviceSettings>();
	return FString::Printf(TEXT("%s:%d"), *DeviceSettings->IpAddress, DeviceSettings->Port);
}

bool ULiveLinkSoundDeviceBase::SetHardwareId_Implementation(const FString& HardwareID)
{
	ULiveLinkSoundDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkSoundDeviceSettings>();

	FIPv4Endpoint OutEndpoint;
	if (FIPv4Endpoint::Parse(HardwareID, OutEndpoint))
	{
		DeviceSettings->IpAddress = OutEndpoint.Address.ToString();
		DeviceSettings->Port = OutEndpoint.Port;
	}

	return true;
}

bool ULiveLinkSoundDeviceBase::CanSetHardwareId_Implementation()
{
	return ConnectionStatus == ELiveLinkDeviceConnectionStatus::Disconnected;
}

bool ULiveLinkSoundDeviceBase::Connect_Implementation()
{
	if (ConnectionStatus != ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		return false;
	}

	ConnectionStatus = ELiveLinkDeviceConnectionStatus::Connecting;
	SetConnectionStatus(ConnectionStatus);

	// Test connection with timecode command
	SendCommand(TEXT("tmcode"), [this](bool bSuccess, const FString& Response)
	{
		if (!bSuccess)
		{
			UE_LOGF(LogTemp, Warning, "Failed to connect to Sound Devices");
			ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnected;
			SetConnectionStatus(ConnectionStatus);
			return;
		}

		UE_LOGF(LogTemp, Log, "Connected to Sound Devices, timecode: %ls", *Response);

		// Set filename format to Scene-Take
		SetSetting(TEXT("FileNameFormat"), TEXT("Scene-Take"), [](bool bSuccess) {});

		// Set reel to today's date
		FString DateStr = FDateTime::Now().ToString(TEXT("%Y%m%d"));
		SetSetting(TEXT("ReelName"), DateStr, [](bool bSuccess) {});

		// Configure all drives for recording
		ConfigureDrivesForRecording();

		// Mark as connected
		ConnectionStatus = ELiveLinkDeviceConnectionStatus::Connected;
		SetConnectionStatus(ConnectionStatus);

		UE_LOGF(LogTemp, Log, "Sound Devices configured and ready");
	});

	return true;
}

bool ULiveLinkSoundDeviceBase::Disconnect_Implementation()
{
	if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		return false;
	}

	ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnecting;
	SetConnectionStatus(ConnectionStatus);

	// Stop any active recording
	if (bIsRecording)
	{
		StopRecording_Implementation();
	}

	// Clear polling state
	CurrentPollingOperation = EPollingOperation::None;
	bPollingRequestInFlight = false;

	// Clear cached auth
	bHasCachedChallenge = false;

	ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnected;
	SetConnectionStatus(ConnectionStatus);

	UE_LOGF(LogTemp, Log, "Disconnected from Sound Devices");

	return true;
}

//~ End ILiveLinkDeviceCapability_Connection interface

//~ Begin ILiveLinkDeviceCapability_Recording interface

bool ULiveLinkSoundDeviceBase::StartRecording_Implementation()
{
	if (ConnectionStatus != ELiveLinkDeviceConnectionStatus::Connected || bIsRecording)
	{
		return false;
	}

	ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
	const FString Slate = Session.GetSlateName();
	const int32 Take = Session.GetTakeNumber();

	UE_LOGF(LogTemp, Log, "Starting recording: %ls Take %d", *Slate, Take);

	// Set reel to today's date
	FString DateStr = FDateTime::Now().ToString(TEXT("%Y%m%d"));
	SetSetting(TEXT("ReelName"), DateStr, [](bool) {});

	// Set slate and take
	SetSetting(TEXT("SceneName"), Slate, [](bool) {});
	SetSetting(TEXT("TakeNumber"), FString::FromInt(Take), [](bool) {});

	// Ensure drives are in record mode
	ConfigureDrivesForRecording();

	// Send record command
	SendCommand(TEXT("settransport/rec"), [](bool, const FString&) {});

	// Start polling for "rec" state
	CurrentPollingOperation = EPollingOperation::WaitingForRecording;
	PollingStartTime = FPlatformTime::Seconds();
	bPollingRequestInFlight = false;

	return true;
}

bool ULiveLinkSoundDeviceBase::StopRecording_Implementation()
{
	if (ConnectionStatus != ELiveLinkDeviceConnectionStatus::Connected || !bIsRecording)
	{
		return false;
	}

	UE_LOGF(LogTemp, Log, "Stopping recording...");

	// Send stop command
	SendCommand(TEXT("settransport/stop"), [](bool, const FString&) {});

	// Start polling for "stop" state
	CurrentPollingOperation = EPollingOperation::WaitingForIdle;
	PollingStartTime = FPlatformTime::Seconds();
	bPollingRequestInFlight = false;

	return true;
}

bool ULiveLinkSoundDeviceBase::IsRecording_Implementation() const
{
	return bIsRecording;
}

//~ End ILiveLinkDeviceCapability_Recording interface

//~ Private Methods

FString ULiveLinkSoundDeviceBase::GetBaseUrl() const
{
	const ULiveLinkSoundDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkSoundDeviceSettings>();
	return FString::Printf(TEXT("http://%s:%d/sounddevices"), *DeviceSettings->IpAddress, DeviceSettings->Port);
}

FString ULiveLinkSoundDeviceBase::GetCommandUrl(const FString& Command) const
{
	return GetBaseUrl() + TEXT("/") + Command;
}

FString ULiveLinkSoundDeviceBase::GetRequestURI(const FString& Command) const
{
	return TEXT("/sounddevices/") + Command;
}

TSharedRef<IHttpRequest> ULiveLinkSoundDeviceBase::CreateRequest(const FString& Command)
{
	// Sound devices are usually local on the network so 1.0 should be a very long time.
	const float HttpRequestTimeout = 1.0f;
	TSharedRef<IHttpRequest> Request = FHttpModule::Get().CreateRequest();
	Request->SetURL(GetCommandUrl(Command));
	Request->SetVerb(TEXT("GET"));
	Request->SetTimeout(HttpRequestTimeout);
	return Request;
}

void ULiveLinkSoundDeviceBase::SendCommand(const FString& Command, TFunction<void(bool bSuccess, const FString& Response)> OnComplete)
{
	TWeakObjectPtr<ULiveLinkSoundDeviceBase> WeakThis(this);

	// If we have a cached challenge, use it directly
	if (bHasCachedChallenge)
	{
		RetryWithAuth(Command, CachedChallenge, OnComplete);
		return;
	}

	// Phase 1: Initial request (expect 401)
	TSharedRef<IHttpRequest> Request = CreateRequest(Command);
	Request->OnProcessRequestComplete().BindLambda(
		[WeakThis, Command, OnComplete](FHttpRequestPtr Req, FHttpResponsePtr Resp, bool bWasSuccessful)
		{
			if (!WeakThis.IsValid())
			{
				return;
			}

			if (bWasSuccessful && Resp.IsValid())
			{
				if (Resp->GetResponseCode() == 401)
				{
					// Phase 2: Parse challenge and retry with auth
					FString AuthHeader = Resp->GetHeader(TEXT("WWW-Authenticate"));
					FDigestChallenge Challenge;

					if (FHttpDigestAuthHelper::ParseDigestChallenge(AuthHeader, Challenge))
					{
						// Cache challenge for future requests
						WeakThis->CachedChallenge = Challenge;
						WeakThis->bHasCachedChallenge = true;

						WeakThis->RetryWithAuth(Command, Challenge, OnComplete);
					}
					else
					{
						UE_LOGF(LogTemp, Warning, "Failed to parse Digest challenge");
						if (OnComplete)
						{
							OnComplete(false, FString());
						}
					}
				}
				else if (Resp->GetResponseCode() == 200)
				{
					// Success without auth (unlikely for Sound Devices)
					if (OnComplete)
					{
						OnComplete(true, Resp->GetContentAsString());
					}
				}
				else
				{
					UE_LOGF(LogTemp, Warning, "HTTP error: %d", Resp->GetResponseCode());
					if (OnComplete)
					{
						OnComplete(false, FString());
					}
				}
			}
			else
			{
				UE_LOGF(LogTemp, Warning, "HTTP request failed for command: %ls", *Command);
				if (OnComplete)
				{
					OnComplete(false, FString());
				}
			}
		});

	Request->ProcessRequest();
}

void ULiveLinkSoundDeviceBase::RetryWithAuth(const FString& Command, const FDigestChallenge& Challenge, TFunction<void(bool, const FString&)> OnComplete)
{
	const ULiveLinkSoundDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkSoundDeviceSettings>();
	TWeakObjectPtr<ULiveLinkSoundDeviceBase> WeakThis(this);

	TSharedRef<IHttpRequest> Request = CreateRequest(Command);

	FString AuthValue = FHttpDigestAuthHelper::GenerateDigestResponse(
		DeviceSettings->Username,
		DeviceSettings->Password,
		Request->GetVerb(),
		GetRequestURI(Command),
		Challenge);

	Request->SetHeader(TEXT("Authorization"), AuthValue);
	Request->OnProcessRequestComplete().BindLambda(
		[WeakThis, OnComplete](FHttpRequestPtr Req, FHttpResponsePtr Resp, bool bWasSuccessful)
		{
			if (!WeakThis.IsValid())
			{
				return;
			}

			if (bWasSuccessful && Resp.IsValid() && Resp->GetResponseCode() == 200)
			{
				if (OnComplete)
				{
					OnComplete(true, Resp->GetContentAsString());
				}
			}
			else
			{
				if (Resp.IsValid())
				{
					UE_LOGF(LogTemp, Warning, "Authenticated request failed: %d", Resp->GetResponseCode());
				}
				if (OnComplete)
				{
					OnComplete(false, FString());
				}
			}
		});

	Request->ProcessRequest();
}

void ULiveLinkSoundDeviceBase::GetTransportState(TFunction<void(bool bSuccess, const FString& StateText)> OnComplete)
{
	SendCommand(TEXT("transport"), [OnComplete](bool bSuccess, const FString& Response)
	{
		if (!bSuccess)
		{
			if (OnComplete)
			{
				OnComplete(false, FString());
			}
			return;
		}

		// Parse JSON: {"Transport": "rec"|"stop"}
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Response);

		if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
		{
			FString TransportState;
			if (JsonObject->TryGetStringField(TEXT("Transport"), TransportState))
			{
				if (OnComplete)
				{
					OnComplete(true, TransportState);
				}
				return;
			}
		}

		UE_LOGF(LogTemp, Warning, "Failed to parse transport state from response: %ls", *Response);
		if (OnComplete)
		{
			OnComplete(false, FString());
		}
	});
}

void ULiveLinkSoundDeviceBase::SetSetting(const FString& Key, const FString& Value, TFunction<void(bool bSuccess)> OnComplete)
{
	// URL encode the value
	FString EncodedValue = FGenericPlatformHttp::UrlEncode(Value);
	FString Command = FString::Printf(TEXT("setsetting/%s=%s"), *Key, *EncodedValue);

	SendCommand(Command, [OnComplete](bool bSuccess, const FString& Response)
	{
		if (OnComplete)
		{
			OnComplete(bSuccess);
		}
	});
}

void ULiveLinkSoundDeviceBase::ConfigureDrivesForRecording()
{
	// Set all 4 drives to Record mode (not File Transfer)
	for (int32 i = 1; i <= 4; i++)
	{
		FString Key = FString::Printf(TEXT("RecordToDrive%d"), i);
		SetSetting(Key, TEXT("Record"), [](bool) {});
	}
}

void ULiveLinkSoundDeviceBase::PollTransportState()
{
	if (bPollingRequestInFlight)
	{
		return;
	}

	// Check timeout
	if (FPlatformTime::Seconds() - PollingStartTime > PollingTimeoutSeconds)
	{
		UE_LOGF(LogTemp, Warning, "Transport state polling timeout");
		CurrentPollingOperation = EPollingOperation::None;
		bPollingRequestInFlight = false;
		return;
	}

	bPollingRequestInFlight = true;

	TWeakObjectPtr<ULiveLinkSoundDeviceBase> WeakThis(this);
	GetTransportState([WeakThis](bool bSuccess, const FString& StateText)
	{
		if (!WeakThis.IsValid())
		{
			return;
		}

		WeakThis->OnTransportStateReceived(bSuccess, StateText);
	});
}

void ULiveLinkSoundDeviceBase::OnTransportStateReceived(bool bSuccess, const FString& StateText)
{
	bPollingRequestInFlight = false;

	if (!bSuccess)
	{
		CurrentPollingOperation = EPollingOperation::None;
		return;
	}

	bool bComplete = false;

	if (CurrentPollingOperation == EPollingOperation::WaitingForRecording)
	{
		if (StateText.Equals(TEXT("rec"), ESearchCase::IgnoreCase))
		{
			bIsRecording = true;
			bComplete = true;
			UE_LOGF(LogTemp, Log, "Recording started");
		}
	}
	else if (CurrentPollingOperation == EPollingOperation::WaitingForIdle)
	{
		if (StateText.Equals(TEXT("stop"), ESearchCase::IgnoreCase))
		{
			bIsRecording = false;
			bComplete = true;
			OnRecordingStopped();
		}
	}

	if (bComplete)
	{
		CurrentPollingOperation = EPollingOperation::None;
	}
}

void ULiveLinkSoundDeviceBase::OnRecordingStopped()
{
	UE_LOGF(LogTemp, Log, "Recording stopped, retrieving file metadata...");

	TWeakObjectPtr<ULiveLinkSoundDeviceBase> WeakThis(this);

	// Get recording file path
	SendCommand(TEXT("invoke/RemoteApi/currentRecordTake()"), [WeakThis](bool bSuccess, const FString& Response)
	{
		if (!WeakThis.IsValid() || !bSuccess)
		{
			return;
		}

		// Parse JSON: {"String": "/HD1/..."}
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Response);

		if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
		{
			FString Path;
			if (JsonObject->TryGetStringField(TEXT("String"), Path))
			{
				// Normalize path
				FString NormalizedPath = NormalizeFilePath(Path);
				UE_LOGF(LogTemp, Log, "Recording path: %ls", *NormalizedPath);

				// Retrieve file metadata
				WeakThis->RetrieveFileMetadata(Path);
			}
		}
	});
}

void ULiveLinkSoundDeviceBase::RetrieveFileMetadata(const FString& FilePath)
{
	FString Command = FString::Printf(TEXT("filedetails%s"), *FilePath);

	SendCommand(Command, [FilePath](bool bSuccess, const FString& Response)
	{
		if (!bSuccess)
		{
			return;
		}

		// Parse JSON: {"FileDetails": {"timecodeStart": "...", "duration": ...}}
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Response);

		if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
		{
			const TSharedPtr<FJsonObject>* FileDetailsPtr;
			if (JsonObject->TryGetObjectField(TEXT("FileDetails"), FileDetailsPtr))
			{
				const TSharedPtr<FJsonObject>& FileDetails = *FileDetailsPtr;

				FString TimecodeStart;
				double Duration = 0.0;

				FileDetails->TryGetStringField(TEXT("timecodeStart"), TimecodeStart);
				FileDetails->TryGetNumberField(TEXT("duration"), Duration);

				UE_LOGF(LogTemp, Log, "File metadata - TC Start: %ls, Duration: %.2fs",
					*TimecodeStart, Duration);
			}
		}
	});
}

FString ULiveLinkSoundDeviceBase::NormalizeFilePath(const FString& Path)
{
	FString Result = Path;

	// Convert /HD{i}/ to /Drive_{i}/
	for (int32 i = 1; i <= 4; i++)
	{
		FString OldPattern = FString::Printf(TEXT("/HD%d/"), i);
		FString NewPattern = FString::Printf(TEXT("/Drive_%d/"), i);
		Result = Result.Replace(*OldPattern, *NewPattern);
	}

	return Result;
}

bool ULiveLinkSoundDeviceBase::DeviceTick(float DeltaTime)
{
	// Handle periodic reconnection attempts when disconnected
	if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		ReconnectionTimeAccumulator += DeltaTime;
		if (ReconnectionTimeAccumulator >= ReconnectionIntervalSeconds)
		{
			ReconnectionTimeAccumulator = 0.0f;
			AttemptReconnection();
		}
	}

	// Handle transport state polling
	if (CurrentPollingOperation != EPollingOperation::None)
	{
		PollTransportState();
	}

	return true; // Continue ticking
}

void ULiveLinkSoundDeviceBase::AttemptReconnection()
{
	if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		UE_LOGF(LogTemp, Log, "Attempting reconnection to Sound Devices...");
		Connect_Implementation();
	}
}

void ULiveLinkSoundDeviceBase::HandleSlateNameChanged(FStringView InSlateName)
{
	if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Connected)
	{
		FString Slate(InSlateName);
		UE_LOGF(LogTemp, Log, "Slate name changed to: %ls", *Slate);
		SetSetting(TEXT("SceneName"), Slate, [](bool) {});
	}
}

void ULiveLinkSoundDeviceBase::HandleTakeNumberChanged(int32 InTakeNumber)
{
	if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Connected)
	{
		UE_LOGF(LogTemp, Log, "Take number changed to: %d", InTakeNumber);
		SetSetting(TEXT("TakeNumber"), FString::FromInt(InTakeNumber), [](bool) {});
	}
}

#undef LOCTEXT_NAMESPACE
