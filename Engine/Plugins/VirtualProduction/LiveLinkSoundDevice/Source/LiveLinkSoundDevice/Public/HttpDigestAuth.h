// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Challenge data parsed from HTTP Digest WWW-Authenticate header
 */
struct FDigestChallenge
{
	FString Realm;
	FString Nonce;
	FString Qop;        // Quality of protection (e.g., "auth", "auth-int")
	FString Opaque;
	FString Algorithm;  // e.g., "MD5", "SHA-256"
};

/**
 * Helper class for HTTP Digest Authentication (RFC 2617)
 */
class FHttpDigestAuthHelper
{
public:
	/**
	 * Parse WWW-Authenticate header from 401 response
	 * @param AuthHeader The WWW-Authenticate header value
	 * @param OutChallenge Parsed challenge data
	 * @return true if parsing succeeded
	 */
	static bool ParseDigestChallenge(const FString& AuthHeader, FDigestChallenge& OutChallenge);

	/**
	 * Generate Authorization header value for authenticated request
	 * @param Username Username for authentication
	 * @param Password Password for authentication
	 * @param Method HTTP method (GET, POST, etc.)
	 * @param URI Request URI path
	 * @param Challenge Challenge data from server
	 * @return Authorization header value (includes "Digest " prefix)
	 */
	static FString GenerateDigestResponse(
		const FString& Username,
		const FString& Password,
		const FString& Method,
		const FString& URI,
		const FDigestChallenge& Challenge);

private:
	/**
	 * Compute MD5 hash of input string
	 * @param Input String to hash
	 * @return Lowercase hex string of MD5 digest
	 */
	static FString ComputeMD5Hash(const FString& Input);

	/**
	 * Generate random nonce for client
	 * @return Hex string nonce
	 */
	static FString GenerateNonce();

	/**
	 * Parse key=value or key="value" pair from header
	 * @param Pair String containing key=value
	 * @param OutKey Extracted key
	 * @param OutValue Extracted value (quotes removed)
	 * @return true if parsing succeeded
	 */
	static bool ParseHeaderPair(const FString& Pair, FString& OutKey, FString& OutValue);
};
