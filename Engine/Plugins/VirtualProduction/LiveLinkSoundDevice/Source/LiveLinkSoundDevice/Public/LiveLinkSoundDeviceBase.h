// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkDevice.h"
#include "LiveLinkDeviceCapability_Connection.h"
#include "LiveLinkDeviceCapability_Recording.h"
#include "LiveLinkSoundDeviceSettings.h"
#include "HttpDigestAuth.h"
#include "Interfaces/IHttpRequest.h"
#include "Containers/Ticker.h"

#include "LiveLinkSoundDeviceBase.generated.h"

/**
 * Concrete implementation for Sound Devices recording devices
 * Provides both Connection and Recording capabilities via REST API with HTTP Digest Authentication
 */
UCLASS(BlueprintType, meta = (DisplayName = "Sound Devices Recorder", ToolTip = "Support for Sound Devices recorders (MixPre, 888, Scorpio)"))
class LIVELINKSOUNDDEVICE_API ULiveLinkSoundDeviceBase : public ULiveLinkDevice
	, public ILiveLinkDeviceCapability_Connection
	, public ILiveLinkDeviceCapability_Recording
{
	GENERATED_BODY()

public:
	//~ Begin ULiveLinkDevice interface
	virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const override;
	virtual EDeviceHealth GetDeviceHealth() const override;
	virtual FText GetHealthText() const override;
	virtual void OnDeviceAdded() override;
	virtual void OnDeviceRemoved() override;
	virtual void OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End ULiveLinkDevice interface

	//~ Begin ILiveLinkDeviceCapability_Connection interface
	virtual ELiveLinkDeviceConnectionStatus GetConnectionStatus_Implementation() const override;
	virtual FString GetHardwareId_Implementation() const override;
	virtual bool SetHardwareId_Implementation(const FString& HardwareID) override;
	virtual bool CanSetHardwareId_Implementation() override;
	virtual bool Connect_Implementation() override;
	virtual bool Disconnect_Implementation() override;
	//~ End ILiveLinkDeviceCapability_Connection interface

	//~ Begin ILiveLinkDeviceCapability_Recording interface
	virtual bool StartRecording_Implementation() override;
	virtual bool StopRecording_Implementation() override;
	virtual bool IsRecording_Implementation() const override;
	//~ End ILiveLinkDeviceCapability_Recording interface

private:
	/** Build base URL for Sound Devices REST API */
	FString GetBaseUrl() const;

	/** Build full command URL */
	FString GetCommandUrl(const FString& Command) const;

	/** Get URI path for authentication (without host/port) */
	FString GetRequestURI(const FString& Command) const;

	/** Send authenticated command to device */
	void SendCommand(const FString& Command, TFunction<void(bool bSuccess, const FString& Response)> OnComplete);

	/** Create HTTP request for a command */
	TSharedRef<class IHttpRequest> CreateRequest(const FString& Command);

	/** Retry request with Digest authentication */
	void RetryWithAuth(const FString& Command, const FDigestChallenge& Challenge, TFunction<void(bool, const FString&)> OnComplete);

	/** Get transport state from device */
	void GetTransportState(TFunction<void(bool bSuccess, const FString& StateText)> OnComplete);

	/** Set device parameter (slate, take, etc.) */
	void SetSetting(const FString& Key, const FString& Value, TFunction<void(bool bSuccess)> OnComplete);

	/** Ensure all drives are in Record mode (not File Transfer) */
	void ConfigureDrivesForRecording();

	/** Poll transport state until target state is reached */
	void PollTransportState();

	/** Called when transport state polling completes */
	void OnTransportStateReceived(bool bSuccess, const FString& StateText);

	/** Called when recording stop is confirmed */
	void OnRecordingStopped();

	/** Retrieve file metadata after recording */
	void RetrieveFileMetadata(const FString& FilePath);

	/** Normalize file path from HD{i} format to Drive_{i} format */
	static FString NormalizeFilePath(const FString& Path);

	/** Main device tick function (handles reconnection and polling) */
	bool DeviceTick(float DeltaTime);

	/** Attempt reconnection if disconnected */
	void AttemptReconnection();

	/** Handle slate name changes from recording session */
	void HandleSlateNameChanged(FStringView InSlateName);

	/** Handle take number changes from recording session */
	void HandleTakeNumberChanged(int32 InTakeNumber);

protected:
	/** Settings class to use for Sound Devices */
	UPROPERTY(EditDefaultsOnly, Category="Live Link Device")
	TSubclassOf<ULiveLinkDeviceSettings> SettingsClass;

	/** Current connection status */
	UPROPERTY()
	ELiveLinkDeviceConnectionStatus ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnected;

	/** Whether the device is currently recording */
	UPROPERTY()
	bool bIsRecording = false;

	/** Ticker handle for main device tick (reconnection and polling) */
	FTSTicker::FDelegateHandle DeviceTickerHandle;

	/** What operation we're polling for */
	enum class EPollingOperation
	{
		None,
		WaitingForRecording,  // Polling for "rec" state
		WaitingForIdle,       // Polling for "stop" state
	};
	EPollingOperation CurrentPollingOperation = EPollingOperation::None;

	/** Time when polling started (for timeout) */
	double PollingStartTime = 0.0;

	/** Time accumulator for reconnection attempts */
	float ReconnectionTimeAccumulator = 0.0f;

	/** Flag to prevent polling request flood on slow connections */
	bool bPollingRequestInFlight = false;

	/** Cached Digest challenge for reuse (optimization) */
	FDigestChallenge CachedChallenge;
	bool bHasCachedChallenge = false;

	/** Polling timeout in seconds */
	static constexpr double PollingTimeoutSeconds = 4.0;

	/** Polling interval in seconds */
	static constexpr float PollingIntervalSeconds = 0.1f;

	/** Reconnection attempt interval in seconds */
	static constexpr float ReconnectionIntervalSeconds = 5.0f;
};
