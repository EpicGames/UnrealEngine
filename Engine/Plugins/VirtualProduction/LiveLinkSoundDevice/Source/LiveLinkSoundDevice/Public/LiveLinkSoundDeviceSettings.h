// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkDevice.h"
#include "LiveLinkSoundDeviceSettings.generated.h"

/**
 * Settings for Sound Devices recording device
 */
UCLASS()
class LIVELINKSOUNDDEVICE_API ULiveLinkSoundDeviceSettings : public ULiveLinkDeviceSettings
{
	GENERATED_BODY()

public:
	/** IP address of the Sound Devices device */
	UPROPERTY(EditAnywhere, Category="Sound Devices", meta=(DisplayName="IP Address"))
	FString IpAddress = TEXT("192.168.1.100");

	/** HTTP port for REST API communication */
	UPROPERTY(EditAnywhere, Category="Sound Devices", AdvancedDisplay)
	uint16 Port = 80;

	/** Username for HTTP Digest Authentication */
	UPROPERTY(EditAnywhere, Category="Sound Devices", AdvancedDisplay)
	FString Username = TEXT("guest");

	/** Password for HTTP Digest Authentication */
	UPROPERTY(EditAnywhere, Category="Sound Devices", AdvancedDisplay)
	FString Password = TEXT("guest");

	/** Device model retrieved from device (read-only) */
	UPROPERTY(VisibleAnywhere, Category="Sound Devices", AdvancedDisplay, meta=(DisplayName="Device Model"))
	FString DeviceModel;
};
