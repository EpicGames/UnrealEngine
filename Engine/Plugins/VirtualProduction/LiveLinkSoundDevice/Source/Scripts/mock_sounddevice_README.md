# Mock Sound Devices Recorder Simulator

A Python-based HTTP server that simulates the REST API of a Sound Devices recorder for testing the Live Link Sound Device plugin without physical hardware.

## Features

- **HTTP Digest Authentication**: Full implementation of RFC 2617 Digest Auth
- **Sound Devices REST API**: Simulates all key endpoints
- **Transport State Machine**: Realistic `stop` ↔ `rec` state transitions
- **Multi-Drive Support**: Simulates 4 drives with configurable modes
- **Recording Metadata**: Generates timecode and duration information
- **Real-time Logging**: Console output shows all commands received

## Requirements

- Python 3.6 or higher
- No external dependencies (uses standard library only)

## Usage

### Basic Usage

```bash
# Run on port 8080 (recommended for testing)
python mock_sounddevice_device.py --port 8080

# Run on port 80 (requires admin/sudo)
sudo python mock_sounddevice_device.py --port 80
```

### Command-Line Options

```bash
python mock_sounddevice_device.py [OPTIONS]

Options:
  --port PORT         HTTP port (default: 8080)
  --ip IP            Bind IP address (default: 0.0.0.0)
  --username USER    Auth username (default: guest)
  --password PASS    Auth password (default: guest)
```

### Examples

```bash
# Listen on localhost only
python mock_sounddevice_device.py --ip 127.0.0.1 --port 8080

# Custom credentials
python mock_sounddevice_device.py --username admin --password secret --port 8080

# Production-like setup (requires elevated permissions)
sudo python mock_sounddevice_device.py --port 80 --ip 192.168.1.100
```

## Testing with Live Link Hub

1. **Start the Mock Server**:
   ```bash
   cd D:\work\fn\Engine\Plugins\VirtualProduction\LiveLinkDeviceSoundDevice\Source\Scripts
   python mock_sounddevice_device.py --port 8080
   ```

   You should see:
   ```
   ============================================================
   Mock Sound Devices Recorder Simulator
   ============================================================
   Listening on: http://0.0.0.0:8080
   Authentication: guest / guest
   Realm: Sound Devices API

   Waiting for connections from Live Link Hub...
   Press Ctrl+C to stop
   ============================================================
   ```

2. **Launch Live Link Hub**:
   - Start Unreal Editor or Live Link Hub standalone

3. **Add Sound Devices Device**:
   - Click "Add Device" in Live Link Hub
   - Select "Sound Devices Recorder" from the device type list
   - Device appears in the device table

4. **Configure Device Settings**:
   - Select the device row
   - Edit settings in Details panel:
     - **IP Address**: `127.0.0.1`
     - **Port**: `8080`
     - **Username**: `guest`
     - **Password**: `guest`

5. **Connect to Device**:
   - Click "Connect" button in device row
   - Device status should change: Disconnected → Connecting → Connected - Ready
   - Mock server logs connection:
     ```
     [SETTING] FileNameFormat = Scene-Take
     [SETTING] ReelName = 20260309
     [SETTING] Drive 1 = Record
     [SETTING] Drive 2 = Record
     [SETTING] Drive 3 = Record
     [SETTING] Drive 4 = Record
     ```

6. **Test Recording**:
   - Set slate and take in Live Link Hub recording session
   - Click "Start Recording" button
   - Mock server logs:
     ```
     [SETTING] ReelName = 20260309
     [SETTING] SceneName = MyScene
     [SETTING] TakeNumber = 1
     [SETTING] Drive 1 = Record
     [SETTING] Drive 2 = Record
     [SETTING] Drive 3 = Record
     [SETTING] Drive 4 = Record

     [RECORDING STARTED]
       Path: /HD1/20260309/MyScene_Take1.mov
       Scene: MyScene
       Take: 1
     ```
   - Device health changes to "Good" (green)
   - Health text shows "Recording"

7. **Test Stop Recording**:
   - Click "Stop Recording" button
   - Mock server logs:
     ```
     [RECORDING STOPPED]
       Path: /HD1/20260309/MyScene_Take1.mov
     ```
   - Device health returns to "Nominal"
   - Health text shows "Connected - Ready"

8. **Test Slate/Take Updates**:
   - Change slate or take in Live Link Hub
   - Mock server should log:
     ```
     [SETTING] SceneName = NewScene
     [SETTING] TakeNumber = 5
     ```

9. **Test Disconnect**:
   - Click "Disconnect" button
   - Device status changes to "Disconnected"
   - Health shows "Error" (red) - expected for disconnected state

## API Endpoints

The mock server implements these Sound Devices REST API endpoints:

| Endpoint | Response | Description |
|----------|----------|-------------|
| `GET /sounddevices/tmcode` | Plain text: `"01:00:00:00"` | Get current timecode |
| `GET /sounddevices/transport` | JSON: `{"Transport": "stop"\|"rec"}` | Get transport state |
| `GET /sounddevices/settransport/rec` | `"OK"` | Start recording |
| `GET /sounddevices/settransport/stop` | `"OK"` | Stop recording |
| `GET /sounddevices/setsetting/{key}={value}` | `"OK"` | Set device parameter |
| `GET /sounddevices/invoke/RemoteApi/currentRecordTake()` | JSON: `{"String": "/HD1/..."}` | Get recording path |
| `GET /sounddevices/filedetails/{path}` | JSON: `{"FileDetails": {...}}` | Get file metadata |

## HTTP Digest Authentication

The mock server implements full HTTP Digest Authentication (RFC 2617):

### Authentication Flow

1. **Client Request** (no auth):
   ```
   GET /sounddevices/tmcode HTTP/1.1
   ```

2. **Server Challenge** (401 Unauthorized):
   ```
   HTTP/1.1 401 Unauthorized
   WWW-Authenticate: Digest realm="Sound Devices API", nonce="abc123", qop="auth", opaque="xyz789"
   ```

3. **Client Response** (with computed digest):
   ```
   GET /sounddevices/tmcode HTTP/1.1
   Authorization: Digest username="guest", realm="Sound Devices API", nonce="abc123",
                         uri="/sounddevices/tmcode", qop=auth, nc=00000001, cnonce="def456",
                         response="computed_md5_hash"
   ```

4. **Server Success**:
   ```
   HTTP/1.1 200 OK
   Content-Type: text/plain

   01:00:00:00
   ```

### Digest Computation

The server verifies the response using MD5:

```
HA1 = MD5(username:realm:password)
HA2 = MD5(method:uri)
Response = MD5(HA1:nonce:nc:cnonce:qop:HA2)
```

## State Machine

### Transport States

```
    stop ──┐
     ↑     │
     │     ↓
     └──  rec
```

- **stop**: Idle, ready to record
- **rec**: Currently recording

### Drive Modes

Each of the 4 drives can be in:
- **Record**: Drive available for recording
- **Ethernet File Transfer**: Drive in file transfer mode

## Recording Metadata

When recording stops, the mock server stores metadata:

```json
{
  "FileDetails": {
    "timecodeStart": "01:00:00:00",
    "duration": 123.45,
    "sampleRate": 48000,
    "bitDepth": 24
  }
}
```

## File Path Format

Recordings are stored with paths like:
```
/HD{drive_number}/{reel_name}/{scene_name}_Take{take_number}.mov
```

Example:
```
/HD1/20260309/MyScene_Take1.mov
```

The Live Link plugin normalizes these to:
```
/Drive_1/20260309/MyScene_Take1.mov
```

## Console Output

The mock server provides detailed logging:

```
[SETTING] FileNameFormat = Scene-Take
[SETTING] ReelName = 20260309
[SETTING] SceneName = TestScene
[SETTING] TakeNumber = 3
[SETTING] Drive 1 = Record
[SETTING] Drive 2 = Record
[SETTING] Drive 3 = Record
[SETTING] Drive 4 = Record

[RECORDING STARTED]
  Path: /HD1/20260309/TestScene_Take3.mov
  Scene: TestScene
  Take: 3

[RECORDING STOPPED]
  Path: /HD1/20260309/TestScene_Take3.mov
```

## Troubleshooting

### Connection Failed

If Live Link Hub cannot connect:

1. **Check server is running**:
   ```bash
   python mock_sounddevice_device.py --port 8080
   ```

2. **Verify IP/port in Live Link Hub settings**:
   - IP Address: `127.0.0.1` (localhost)
   - Port: `8080`

3. **Check firewall**:
   - Ensure port 8080 is not blocked
   - On Windows: Check Windows Defender Firewall

4. **Check credentials**:
   - Default: username=`guest`, password=`guest`
   - Match mock server args if changed

### Authentication Errors

If you see 401 Unauthorized errors:

1. **Verify credentials** match server configuration
2. **Check console output** for authentication attempts
3. **Ensure Digest Auth** is properly implemented in plugin

### Recording Not Starting

If recording doesn't start:

1. **Check device is connected** (status = "Connected - Ready")
2. **Verify transport state** in mock server console
3. **Check for error messages** in Live Link Hub log

## Limitations

### Mock Server

- Single-threaded (handles one request at a time)
- No persistent storage (state resets on restart)
- Simplified file metadata (fixed values)
- No actual audio file generation
- No multi-client support

### Not Implemented

- File transfer mode functionality
- mDNS/Bonjour discovery
- Individual track configuration
- Real timecode generation
- Advanced recording formats

## Real Device Testing

After testing with the mock server, test with actual Sound Devices hardware:

1. **Supported Devices**:
   - MixPre series (MixPre-3, MixPre-6, MixPre-10)
   - 8-Series (888, 833, 888)
   - Scorpio

2. **Network Configuration**:
   - Connect device to network
   - Note device IP address (check device menu)
   - Ensure device API is enabled

3. **Live Link Hub Configuration**:
   - Use device's actual IP address
   - Port is typically `80`
   - Default credentials: `guest` / `guest`

4. **Verify API Access**:
   ```bash
   # Test from command line
   curl -u guest:guest --digest http://{device_ip}/sounddevices/tmcode
   ```

## Support

For issues or questions:
- Check Live Link Hub logs
- Review mock server console output
- Verify network connectivity
- Ensure Sound Devices firmware is up to date (for real devices)
