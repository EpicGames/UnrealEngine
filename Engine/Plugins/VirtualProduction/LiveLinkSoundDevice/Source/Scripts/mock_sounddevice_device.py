#!/usr/bin/env python3
"""
Mock Sound Devices Recorder Simulator

Simulates the REST API of a Sound Devices recorder for testing the Live Link Sound Device plugin.
Implements HTTP Digest Authentication and the key endpoints needed for recording control.

Usage:
    python mock_sounddevice_device.py [--port PORT] [--ip IP]

Example:
    python mock_sounddevice_device.py --port 8080 --ip 127.0.0.1
"""

import argparse
import json
import hashlib
import secrets
import time
import sys
from http.server import HTTPServer, BaseHTTPRequestHandler
from urllib.parse import urlparse, unquote, parse_qs
from threading import Lock


class SoundDeviceState:
    """Manages the state of the mock Sound Devices recorder"""

    def __init__(self):
        self.lock = Lock()

        # Device state
        self.transport_state = "stop"  # "rec", "stop"
        self.scene_name = "DefaultScene"
        self.take_number = "1"
        self.reel_name = "20260101"
        self.file_name_format = "Scene-Take"
        self.current_recording_path = ""

        # Drive configuration (1-4)
        self.drive_modes = {
            1: "Record",
            2: "Record",
            3: "Record",
            4: "Record"
        }

        # Recording metadata storage
        self.recordings = {}  # path -> {timecodeStart, duration}

    def start_recording(self):
        """Start recording"""
        with self.lock:
            if self.transport_state == "stop":
                self.transport_state = "rec"

                # Generate recording path based on format
                if self.file_name_format == "Scene-Take":
                    filename = f"{self.scene_name}_Take{self.take_number}.mov"
                else:
                    filename = f"recording_{int(time.time())}.mov"

                # Place on first drive in record mode
                drive_num = 1
                for d, mode in self.drive_modes.items():
                    if mode == "Record":
                        drive_num = d
                        break

                self.current_recording_path = f"/HD{drive_num}/{self.reel_name}/{filename}"

                print(f"\n[RECORDING STARTED]")
                print(f"  Path: {self.current_recording_path}")
                print(f"  Scene: {self.scene_name}")
                print(f"  Take: {self.take_number}")
                sys.stdout.flush()

                return True
            return False

    def stop_recording(self):
        """Stop recording"""
        with self.lock:
            if self.transport_state == "rec":
                self.transport_state = "stop"

                # Store recording metadata
                if self.current_recording_path:
                    self.recordings[self.current_recording_path] = {
                        "timecodeStart": "01:00:00:00",
                        "duration": 123.45,
                        "sampleRate": 48000,
                        "bitDepth": 24
                    }

                print(f"\n[RECORDING STOPPED]")
                print(f"  Path: {self.current_recording_path}")
                sys.stdout.flush()

                return True
            return False

    def set_setting(self, key, value):
        """Set a device setting"""
        with self.lock:
            if key == "FileNameFormat":
                self.file_name_format = value
                print(f"[SETTING] FileNameFormat = {value}")
                sys.stdout.flush()
            elif key == "ReelName":
                self.reel_name = value
                print(f"[SETTING] ReelName = {value}")
                sys.stdout.flush()
            elif key == "SceneName":
                self.scene_name = value
                print(f"[SETTING] SceneName = {value}")
                sys.stdout.flush()
            elif key == "TakeNumber":
                self.take_number = value
                print(f"[SETTING] TakeNumber = {value}")
                sys.stdout.flush()
            elif key.startswith("RecordToDrive"):
                drive_num = int(key[-1])  # Extract drive number
                self.drive_modes[drive_num] = value
                print(f"[SETTING] Drive {drive_num} = {value}")
                sys.stdout.flush()
            else:
                print(f"[SETTING] {key} = {value} (unhandled)")
                sys.stdout.flush()


class DigestAuthHandler:
    """Implements HTTP Digest Authentication (RFC 2617)"""

    def __init__(self, username="guest", password="guest"):
        self.username = username
        self.password = password
        self.realm = "Sound Devices API"
        self.nonce = secrets.token_hex(16)
        self.opaque = secrets.token_hex(16)

    def generate_challenge(self):
        """Generate WWW-Authenticate header"""
        return f'Digest realm="{self.realm}", nonce="{self.nonce}", qop="auth", opaque="{self.opaque}"'

    def verify_response(self, auth_header, method, uri):
        """Verify Authorization header"""
        if not auth_header or not auth_header.startswith("Digest "):
            return False

        # Parse Authorization header
        auth_str = auth_header[7:]  # Remove "Digest "
        parts = {}

        # Simple parser for key=value or key="value"
        for part in auth_str.split(","):
            part = part.strip()
            if "=" in part:
                key, value = part.split("=", 1)
                key = key.strip()
                value = value.strip().strip('"')
                parts[key] = value

        # Verify required fields
        if "response" not in parts or "nonce" not in parts:
            return False

        # Compute expected response
        # HA1 = MD5(username:realm:password)
        ha1_input = f"{self.username}:{self.realm}:{self.password}"
        ha1 = hashlib.md5(ha1_input.encode()).hexdigest()

        # HA2 = MD5(method:uri)
        ha2_input = f"{method}:{uri}"
        ha2 = hashlib.md5(ha2_input.encode()).hexdigest()

        # Response
        if "qop" in parts and parts["qop"] == "auth":
            # qop=auth mode
            nc = parts.get("nc", "00000001")
            cnonce = parts.get("cnonce", "")
            response_input = f"{ha1}:{parts['nonce']}:{nc}:{cnonce}:{parts['qop']}:{ha2}"
        else:
            # Legacy mode
            response_input = f"{ha1}:{parts['nonce']}:{ha2}"

        expected_response = hashlib.md5(response_input.encode()).hexdigest()

        return parts["response"] == expected_response


class SoundDeviceRequestHandler(BaseHTTPRequestHandler):
    """HTTP request handler for Sound Devices API"""

    # Shared state and auth handler (set by server)
    device_state = None
    auth_handler = None

    def log_message(self, format, *args):
        """Suppress default logging"""
        pass

    def do_GET(self):
        """Handle GET requests"""
        # Check authentication
        auth_header = self.headers.get("Authorization")

        if not auth_header:
            # Send 401 with challenge
            self.send_response(401)
            self.send_header("WWW-Authenticate", self.auth_handler.generate_challenge())
            self.send_header("Content-Length", "0")
            self.end_headers()
            return

        # Verify authentication
        if not self.auth_handler.verify_response(auth_header, "GET", self.path):
            self.send_error(401, "Unauthorized")
            return

        # Route to command handler
        parsed = urlparse(self.path)
        path_parts = parsed.path.split("/")

        if len(path_parts) >= 2 and path_parts[1] == "sounddevices":
            # Extract command from path
            command = "/".join(path_parts[2:])
            self.handle_command(command)
        else:
            self.send_error(404, "Not Found")

    def handle_command(self, command):
        """Route command to appropriate handler"""
        # Timecode
        if command == "tmcode":
            self.send_text_response("01:00:00:00")

        # Transport state
        elif command == "transport":
            self.send_json_response({"Transport": self.device_state.transport_state})

        # Transport control
        elif command == "settransport/rec":
            self.device_state.start_recording()
            self.send_text_response("OK")

        elif command == "settransport/stop":
            self.device_state.stop_recording()
            self.send_text_response("OK")

        # Settings
        elif command.startswith("setsetting/"):
            param = command[11:]  # Remove "setsetting/"
            if "=" in param:
                key, value = param.split("=", 1)
                value = unquote(value)  # URL decode
                self.device_state.set_setting(key, value)
            self.send_text_response("OK")

        # Recording info
        elif command == "invoke/RemoteApi/currentRecordTake()":
            path = self.device_state.current_recording_path or "/HD1/default.mov"
            self.send_json_response({"String": path})

        # File details
        elif command.startswith("filedetails/"):
            file_path = command[12:]  # Remove "filedetails/"
            if file_path in self.device_state.recordings:
                metadata = self.device_state.recordings[file_path]
                self.send_json_response({"FileDetails": metadata})
            else:
                # Return generic metadata for unknown files
                self.send_json_response({
                    "FileDetails": {
                        "timecodeStart": "01:00:00:00",
                        "duration": 60.0,
                        "sampleRate": 48000,
                        "bitDepth": 24
                    }
                })

        else:
            self.send_error(404, f"Unknown command: {command}")

    def send_text_response(self, text):
        """Send plain text response"""
        response = text.encode()
        self.send_response(200)
        self.send_header("Content-Type", "text/plain")
        self.send_header("Content-Length", str(len(response)))
        self.end_headers()
        self.wfile.write(response)

    def send_json_response(self, data):
        """Send JSON response"""
        response = json.dumps(data).encode()
        self.send_response(200)
        self.send_header("Content-Type", "application/json")
        self.send_header("Content-Length", str(len(response)))
        self.end_headers()
        self.wfile.write(response)


def main():
    parser = argparse.ArgumentParser(description="Mock Sound Devices Recorder Simulator")
    parser.add_argument("--port", type=int, default=8080, help="HTTP port (default: 8080)")
    parser.add_argument("--ip", type=str, default="0.0.0.0", help="Bind IP address (default: 0.0.0.0)")
    parser.add_argument("--username", type=str, default="guest", help="Auth username (default: guest)")
    parser.add_argument("--password", type=str, default="guest", help="Auth password (default: guest)")
    args = parser.parse_args()

    # Force unbuffered output for immediate console visibility
    sys.stdout.reconfigure(line_buffering=True)
    sys.stderr.reconfigure(line_buffering=True)

    # Create shared state and auth handler
    device_state = SoundDeviceState()
    auth_handler = DigestAuthHandler(username=args.username, password=args.password)

    # Assign to handler class
    SoundDeviceRequestHandler.device_state = device_state
    SoundDeviceRequestHandler.auth_handler = auth_handler

    # Create and start server
    server = HTTPServer((args.ip, args.port), SoundDeviceRequestHandler)

    print("=" * 60)
    print("Mock Sound Devices Recorder Simulator")
    print("=" * 60)
    print(f"Listening on: http://{args.ip}:{args.port}")
    print(f"Authentication: {args.username} / {args.password}")
    print(f"Realm: {auth_handler.realm}")
    print()
    print("Waiting for connections from Live Link Hub...")
    print("Press Ctrl+C to stop")
    print("=" * 60)
    print()

    try:
        server.serve_forever()
    except KeyboardInterrupt:
        print("\n\nShutting down...")
        server.shutdown()


if __name__ == "__main__":
    main()
