@echo off
REM Helper script to run the mock Sound Devices server with visible console output
REM Usage: run_mock.bat [port]
REM Example: run_mock.bat 8080

setlocal
set PORT=8080

if not "%1"=="" set PORT=%1

echo ================================================================
echo Starting Mock Sound Devices Recorder on port %PORT%
echo ================================================================
echo.
echo Press Ctrl+C to stop the server
echo.

python -u mock_sounddevice_device.py --port %PORT%

endlocal
